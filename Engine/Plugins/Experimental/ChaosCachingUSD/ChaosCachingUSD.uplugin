{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Chaos Caching USD",
	"Description": "Adds support for using USD files for caching Chaos flesh simulations",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Modules": [
		{
			"Name": "ChaosCachingUSD",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ChaosCaching",
			"Enabled": true
		},
		{
			"Name": "USDImporter",
			"Enabled": true
		}
	]
}
