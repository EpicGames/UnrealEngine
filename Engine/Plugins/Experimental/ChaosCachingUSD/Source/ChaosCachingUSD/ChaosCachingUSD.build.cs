// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using System.IO;
using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
	public class ChaosCachingUSD : ModuleRules
	{
		public ChaosCachingUSD(ReadOnlyTargetRules Target) : base(Target)
		{
			bUseRTTI = true;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
				"Core",
				"CoreUObject",
				"Engine",
				"Projects",
				"RHI",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"UnrealUSDWrapper",
				"USDClasses",
				"USDUtilities",
				}
			);

			UnrealUSDWrapper.CheckAndSetupUsdSdk(Target, this);
		}
	}
}