// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCachingUSD/Module.h"
#include "Modules/ModuleManager.h"

#if USE_USD_SDK
#include "ChaosCachingUSD/UEUsdGeomTetMesh.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

#include "UnrealUSDWrapper.h"
#include "USDMemory.h"

#include "USDIncludesStart.h"
#include "pxr/usd/sdf/assetPath.h"
#include "pxr/usd/sdf/types.h"
#include "pxr/usd/usd/schemaRegistry.h"
#include "pxr/usd/usd/typed.h"
#include "USDIncludesEnd.h"

IMPLEMENT_MODULE_USD(FChaosCachingUSDModule, ChaosCachingUSD);
#else
IMPLEMENT_MODULE(FChaosCachingUSDModule, ChaosCachingUSD);
#endif // USE_USD_SDK


// UE_DEPRECATED(5.7, "Use LogUsd instead")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
DEFINE_LOG_CATEGORY(LogChaosCacheUSD)
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FChaosCachingUSDModule::StartupModule()
{
#if USE_USD_SDK
	// Register the ChaosCachingUSD plugin with USD.
	IPluginManager& UEPluginManager = IPluginManager::Get();
	FString USDImporterDir = UEPluginManager.FindPlugin(TEXT("USDImporter"))->GetBaseDir();

	const FString ChaosCachingUSDResourcesDir =
		FPaths::ConvertRelativePathToFull(
			FPaths::Combine(
				USDImporterDir, 
				FString(TEXT("ChaosCachingUSD")),
				FString(TEXT("Resources"))));

	UnrealUSDWrapper::RegisterPlugins(ChaosCachingUSDResourcesDir);

	// Register the tetmesh schema
	{
		using namespace pxr;
		FScopedUsdAllocs UEAllocs;  // Use USD memory allocator
		TfType::Define<UEUsdGeomTetMesh, TfType::Bases<UsdGeomMesh> >();
		TfType::AddAlias<UsdSchemaBase, UEUsdGeomTetMesh>("UETetMesh");
	}
#endif // USE_USD_SDK
}
