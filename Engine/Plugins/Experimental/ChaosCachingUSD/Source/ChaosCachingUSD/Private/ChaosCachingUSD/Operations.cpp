// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCachingUSD/Operations.h"
#include "ChaosCachingUSD/UEUsdGeomTetMesh.h"
#include "ChaosCachingUSD/Util.h"

#if USE_USD_SDK

#include "Containers/StringConv.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArray.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"

#include "UnrealUSDWrapper.h"
#include "USDConversionUtils.h"
#include "USDLog.h"
#include "USDMemory.h"

#include "UsdWrappers/SdfLayer.h"
#include "UsdWrappers/SdfPath.h"
#include "UsdWrappers/UsdAttribute.h"
#include "UsdWrappers/UsdPrim.h"
#include "UsdWrappers/UsdStage.h"

#include "USDIncludesStart.h"
	#include "pxr/usd/sdf/primSpec.h"
	#include "pxr/usd/usd/stageCacheContext.h"
	#include "pxr/usd/usdGeom/mesh.h"
	#include "pxr/usd/usdGeom/pointBased.h"
	#include "pxr/usd/usdGeom/tokens.h"
	#include "pxr/usd/usdUtils/stageCache.h"
#include "USDIncludesEnd.h"

bool 
UE::ChaosCachingUSD::NewStage(const FString& StageName, UE::FUsdStage& UsdStage)
{
	if (FPaths::FileExists(StageName))
	{
		UE_LOGF(LogUsd, Error, "Cannot clobber existing file: '%ls'", *StageName);
		return false;
	}

	// Verify that StageName is a supported format.  Note that there's GetAllSupportedFileFormats()
	// and GetNativeFileFormats().  In this case, I think since we're creating a stage, we want
	// a native format.
	FString FileFormat = FPaths::GetExtension(StageName, false); // no dot
	TArray<FString> SupportedFormats = UnrealUSDWrapper::GetNativeFileFormats();
	bool bFound = false;
	for (const FString& Ext : SupportedFormats)
	{
		if (FileFormat == Ext)
		{
			bFound = true;
			break;
		}
	}
	if (!bFound)
	{
		FString Formats;
		for (int32 i=0; i < SupportedFormats.Num(); i++)
		{
			const FString& Ext = SupportedFormats[i];
			Formats.Append(Ext);
			if (i < SupportedFormats.Num() - 1) Formats.AppendChar(' ');
		}

		UE_LOGF(LogUsd, Error, 
			"Failed to create new USD stage: '%ls', file format '%ls' is not native to USD. Supported formats: '%ls'",
			*StageName, *FileFormat, *Formats);
		return false;
	}

	// USD caches all stages you open/create, unless you tell it not to.
	// For convenience, we're using the global/default stage cache.
	UsdStage = UnrealUSDWrapper::NewStage(*StageName);
	if (!UsdStage)
	{
		UE_LOGF(LogUsd, Error, "Failed to create new USD stage: '%ls'", *StageName);
		return false;
	}

	// TODO: Ryan
	//UsdUtils::SetUsdStageMetersPerUnit(UsdStage, ...);
	UsdUtils::SetUsdStageUpAxis(UsdStage, pxr::UsdGeomTokens->z);
	return true;
}

bool 
UE::ChaosCachingUSD::OpenStage(const FString& StageName, UE::FUsdStage& UsdStage)
{
	if (!FPaths::FileExists(StageName))
	{
		FPlatformFileManager& FileManager = FPlatformFileManager::Get();
		IPlatformFile& PlatformFile = FileManager.GetPlatformFile();
		if (!PlatformFile.FileExists(*StageName))
		{
			UE_LOGF(LogUsd, Error, "File not found: '%ls'", *StageName);
			return false;
		}
	}

	// USD caches all stages you open/create, unless you tell it not to.
	// For convenience, we're using the global/default stage cache.
	UsdStage = UnrealUSDWrapper::OpenStage(
			*StageName,						
			EUsdInitialLoadSet::LoadAll,	// load payload?
			true,							// use stage cache
			false);							// force reload from disk
	if (!UsdStage)
	{
		UE_LOGF(LogUsd, Error, "Failed to open USD stage: '%ls'", *StageName);
		return false;
	}
	return true;
}

bool 
UE::ChaosCachingUSD::SaveStage(UE::FUsdStage& Stage, const double FirstFrame, const double LastFrame)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	if (FirstFrame != -TNumericLimits<double>::Max() && LastFrame != -TNumericLimits<double>::Max())
	{
		const double StageStart = Stage.GetStartTimeCode();
		const double StageEnd = Stage.GetEndTimeCode();
		const bool bHasRange = static_cast<pxr::UsdStageRefPtr>(Stage)->HasAuthoredTimeCodeRange();
		if (!bHasRange || FirstFrame < StageStart)
		{
			Stage.SetStartTimeCode(FirstFrame);
		}
		if (!bHasRange || LastFrame > StageEnd)
		{
			Stage.SetEndTimeCode(LastFrame);
		}
	}
	if (UE::FSdfLayer RootLayer = Stage.GetRootLayer())
	{
		UE_LOGF(LogUsd, Log, "Saving USD file: '%ls'", *RootLayer.GetRealPath());
		if (!RootLayer.Save())
		{
			UE_LOGF(LogUsd, Error, "Failed to save USD file: '%ls'", *RootLayer.GetRealPath());
			return false;
		}
		return true;
	}
	else
	{
		UE_LOGF(LogUsd, Error, "USD Stage has no root layer.");
		return false;
	}
}

bool
UE::ChaosCachingUSD::CloseStage(const UE::FUsdStage& Stage)
{
	UnrealUSDWrapper::EraseStageFromCache(Stage);
	return true;
}

bool 
UE::ChaosCachingUSD::CloseStage(const FString& StageName)
{
	TArray<UE::FUsdStage> CachedStages = UnrealUSDWrapper::GetAllStagesFromCache();
	for (UE::FUsdStage& Stage : CachedStages)
	{
		if (Stage.GetRootLayer().GetRealPath() == StageName)
		{
			return ChaosCachingUSD::CloseStage(Stage);
		}
	}
	return true;
}

void
UE::ChaosCachingUSD::GenerateValueClipStageNames(
	const FString& ParentName, 
	FString& TopologyName, 
	FString& TimeVaryingTemplate)
{
	FScopedUnrealAllocs UEAllocs; // Use UE memory allocator
	std::string ParentNameStr(TCHAR_TO_ANSI(*ParentName));
	std::string TopologyNameStr = pxr::UsdUtilsGenerateClipTopologyName(ParentNameStr);
	TopologyName = FString(ANSI_TO_TCHAR(TopologyNameStr.c_str()));
	// Usd supports an integral template format, and a floating point.  We're using the
	// float format here.  The integral format is "path/name.####.usd".
	TimeVaryingTemplate = TopologyName.Replace(TEXT("topology"), TEXT("###.###"));
}

FString
UE::ChaosCachingUSD::GenerateValueClipTimeVaryingStageName(
	const FString& TimeVaryingTemplate, 
	const double Time)
{
	FScopedUnrealAllocs UEAllocs; // Use UE memory allocator
	if (Time == -TNumericLimits<double>::Max())
	{
		return TimeVaryingTemplate.Replace(TEXT("###.###"), TEXT("default"));
	}
	FString Template = TimeVaryingTemplate.Replace(TEXT("###.###"), TEXT("{0}"));
	FString TimeStr = FString::Printf(TEXT("%06.3f"), Time);
	return FString::Format(*Template, { TimeStr });
}

bool
UE::ChaosCachingUSD::NewValueClipsStages(
	const FString& ParentStageName,
	const FString& TopologyStageName,
	UE::FUsdStage& ParentStage,
	UE::FUsdStage& TopologyStage)
{
	if (!ChaosCachingUSD::NewStage(ParentStageName, ParentStage)) return false;
	if (!ChaosCachingUSD::NewStage(TopologyStageName, TopologyStage)) return false;
	return true;
}

bool
UE::ChaosCachingUSD::NewValueClipsFrameStage(
	const FString& TimeVaryingStageTemplate,
	const double Time,
	FString& FrameStageName,
	UE::FUsdStage& FrameStage)
{
	FrameStageName =
		ChaosCachingUSD::GenerateValueClipTimeVaryingStageName(TimeVaryingStageTemplate, Time);
	return ChaosCachingUSD::NewStage(FrameStageName, FrameStage);
}

bool 
UE::ChaosCachingUSD::InitValueClipsTemplate(
	UE::FUsdStage& ParentStage,
	UE::FUsdStage& TopologyStage,
	const FString& ParentStageName,
	const FString& TopologyStageName,
	const FString& TimeVaryingStageTemplate,
	const TArray<FString>& PrimPaths,
	const double StartTime,
	const double EndTime,
	const double Stride)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	UE::FSdfLayer ParentLayer = ParentStage.GetRootLayer();
	UE::FSdfLayer TopologyLayer = ParentStage.GetRootLayer();

	for (const FString& PrimPath : PrimPaths)
	{
		// Get or create the prim.
		pxr::SdfPrimSpecHandle Prim = pxr::SdfCreatePrimInLayer(ParentLayer, UE::FSdfPath(*PrimPath));

		// Get the existing metadata, if it exists.
		pxr::VtDictionary Clips;
		pxr::VtDictionary ClipSetDict;
		if (Prim->HasInfo(pxr::UsdTokens->clips))
		{
			pxr::VtValue PrevClipInfoValue = Prim->GetInfo(pxr::UsdTokens->clips);
			if (PrevClipInfoValue.IsHolding<pxr::VtDictionary>())
			{
				Clips = PrevClipInfoValue.Get<pxr::VtDictionary>();
				auto it = Clips.find(pxr::UsdClipsAPISetNames->default_);
				if (it != Clips.end())
				{
					pxr::VtValue PrevClipsEntryValue = it->second;
					if (PrevClipsEntryValue.IsHolding<pxr::VtDictionary>())
					{
						ClipSetDict = PrevClipsEntryValue.Get<pxr::VtDictionary>();
					}
				}
			}
		}

		// Add the topology layer.
		if (!ParentStageName.IsEmpty() && !TopologyStageName.IsEmpty() && TopologyLayer)
		{
			FString TopologyId = TopologyStageName;
			FPaths::MakePathRelativeTo(TopologyId, *ParentStageName);
			std::string TopologyIdStr(TCHAR_TO_ANSI(*TopologyId));

			// Add the sub layer in the parent pointing to the topology.
			static_cast<pxr::SdfLayerRefPtr>(ParentLayer)->InsertSubLayerPath(
				TopologyIdStr, 0); // 0 means strongest
			
			// Set root layer metadata
			ClipSetDict[pxr::UsdClipsAPIInfoKeys->manifestAssetPath] = 
				pxr::SdfAssetPath(TopologyIdStr);
		}

		// Add time varying stage template.
		if (!TimeVaryingStageTemplate.IsEmpty())
		{
			std::string TimeVaryingStageTemplateStr(TCHAR_TO_ANSI(*TimeVaryingStageTemplate));
			ClipSetDict[pxr::UsdClipsAPIInfoKeys->templateAssetPath] = 
				TimeVaryingStageTemplateStr;
		}

		// Add the prim path.
		std::string PrimPathStr(TCHAR_TO_ANSI(*PrimPath));
		ClipSetDict[pxr::UsdClipsAPIInfoKeys->primPath] = PrimPathStr;

		// Add start and end time.
		if (fabs(StartTime) != TNumericLimits<double>::Max() &&
			fabs(EndTime) != TNumericLimits<double>::Max())
		{
			ClipSetDict[pxr::UsdClipsAPIInfoKeys->templateStartTime] = StartTime;
			ClipSetDict[pxr::UsdClipsAPIInfoKeys->templateEndTime] = EndTime;
		}

		// Add sampling stride.
		if (Stride > 0.0)
		{
			ClipSetDict[pxr::UsdClipsAPIInfoKeys->templateStride] = Stride;
		}

		// Set metadata info on Prim.
		Clips[pxr::UsdClipsAPISetNames->default_] = ClipSetDict;
		Prim->SetInfo(pxr::UsdTokens->clips, pxr::VtValue::Take(Clips));
	}
	ParentLayer.SetStartTimeCode(StartTime);
	ParentLayer.SetEndTimeCode(EndTime);
	return true;
}

bool
UE::ChaosCachingUSD::InitValueClipsTemplate(
	const FString& ParentStageName,
	const FString& TopologyStageName,
	const FString& TimeVaryingStageTemplate,
	const TArray<FString>& PrimPaths,
	const double StartTime,
	const double EndTime,
	const double Stride)
{
	UE::FUsdStage ParentStage;
	if (!ChaosCachingUSD::OpenStage(ParentStageName, ParentStage)) return false;
	UE::FUsdStage TopologyStage;
	if (!ChaosCachingUSD::OpenStage(TopologyStageName, TopologyStage)) return false;

	return ChaosCachingUSD::InitValueClipsTemplate(
		ParentStage,
		TopologyStage,
		ParentStageName,
		TopologyStageName,
		TimeVaryingStageTemplate,
		PrimPaths,
		StartTime,
		EndTime,
		Stride);
}

bool 
UE::ChaosCachingUSD::WriteTetMesh(
	UE::FUsdStage& Stage, 
	const FString& PrimPath,
	const FManagedArrayCollection& Collection, 
	const int32 StructureIndex)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	const TManagedArray<FIntVector4>* Tetrahedron =
		Collection.FindAttribute<FIntVector4>(
			/*FTetrahedralCollection::TetrahedronAttribute*/FName("Tetrahedron"), 
			/*FTetrahedralCollection::TetrahedralGroup*/FName("Tetrahedral"));
	const TManagedArray<int32>* TetrahedronStart =
		Collection.FindAttribute<int32>(
			/*FTetrahedralCollection::TetrahedronStartAttribute*/FName("TetrahedronStart"), 
			/*FGeometryCollection::GeometryGroup*/FName("Tetrahedral"));
	const TManagedArray<int32>* TetrahedronCount =
		Collection.FindAttribute<int32>(
			/*FTetrahedralCollection::TetrahedronCountAttribute*/FName("TetrahedronCount"), 
			/*FGeometryCollection::GeometryGroup*/FName("Geometry"));

	const TManagedArray<FIntVector>* Triangle =
		Collection.FindAttribute<FIntVector>(
			"Indices", /*FGeometryCollection::FacesGroup*/FName("Faces"));
	const TManagedArray<int32>* FacesStart =
		Collection.FindAttribute<int32>(
			"FaceStart", /*FGeometryCollection::GeometryGroup*/FName("Geometry"));
	const TManagedArray<int32>* FacesCount =
		Collection.FindAttribute<int32>(
			"FaceCount", /*FGeometryCollection::GeometryGroup*/FName("Geometry"));

	const TManagedArray<int32>* VertexStart =
		Collection.FindAttribute<int32>(
			"VertexStart", /*FGeometryCollection::GeometryGroup*/FName("Geometry"));
	const TManagedArray<int32>* VertexCount =
		Collection.FindAttribute<int32>(
			"VertexCount", /*FGeometryCollection::GeometryGroup*/FName("Geometry"));

	const TManagedArray<FVector3f>* Vertex =
		Collection.FindAttribute<FVector3f>(
			"Vertex", "Vertices");

	int32 VertStart = 0;
	int32 VertCount = Vertex->Num();
	int32 TetMeshStart = 0;
	int32 TetMeshCount = Tetrahedron->Num();
	int32 TriMeshStart = 0;
	int32 TriMeshCount = Triangle->Num();
	if (StructureIndex != INDEX_NONE)
	{
		if (!VertexStart->IsValidIndex(StructureIndex) ||
			!VertexCount->IsValidIndex(StructureIndex))
		{
			UE_LOGF(LogUsd, Error, "Invalid vertex index: %d, valid range [0, %d).", StructureIndex, VertexStart->Num());
			return false;
		}
		VertStart = (*VertexStart)[StructureIndex];
		VertCount = (*VertexCount)[StructureIndex];
		if (!TetrahedronStart->IsValidIndex(StructureIndex) ||
			!TetrahedronCount->IsValidIndex(StructureIndex))
		{
			UE_LOGF(LogUsd, Error, "Invalid tet mesh index: %d, valid range [0, %d).", StructureIndex, TetrahedronStart->Num());
			return false;
		}
		TetMeshStart = (*TetrahedronStart)[StructureIndex];
		TetMeshCount = (*TetrahedronCount)[StructureIndex];
		if (!FacesStart->IsValidIndex(StructureIndex) ||
			!FacesCount->IsValidIndex(StructureIndex))
		{
			// Not a deal breaker, but highly unusual.
			UE_LOGF(LogUsd, Warning, "Invalid tet mesh faces index: %d, valid range [0, %d),", StructureIndex, FacesStart->Num());
		}
		else
		{
			TriMeshStart = (*FacesStart)[StructureIndex];
			TriMeshCount = (*FacesCount)[StructureIndex];
		}

		UE_LOGF(LogUsd, Log, "Extracting tetrahedral mesh %d of %d from collection; vertex range [%d, %d), tet [%d, %d), tri [%d, %d).",
			StructureIndex, VertexStart->Num(),
			VertStart, VertStart + VertCount,
			TetMeshStart, TetMeshStart + TetMeshCount,
			TriMeshStart, TriMeshStart + TriMeshCount);
	}

	UE::FSdfPath Path(*PrimPath);

	// Author a UsdGeomPointBased override prim.
	if (!UE::ChaosCachingUSD::WritePoints(Stage, PrimPath, -TNumericLimits<double>::Max(), Collection, StructureIndex))
	{
		UE_LOGF(LogUsd, Error, "Failed to create UsdGeomPointBased at path: '%ls' on USD stage: '%ls'.",
			*PrimPath, *Stage.GetRootLayer().GetRealPath());
		return false;
	}

	// Promote to UsdGeomMesh.
	if (pxr::UsdGeomMesh Mesh = pxr::UsdGeomMesh::Define(Stage, Path))
	{
		pxr::UsdPrim Prim = Mesh.GetPrim();

		// Surface tris vertices and counts
		pxr::VtArray<int> VtTriVerts(TriMeshCount * 3);
		int32 k = 0;
		for (int32 i = 0; i < TriMeshCount; i++)
			for (int32 j = 0; j < 3; j++, k++)
				VtTriVerts[k] = (*Triangle)[TriMeshStart + i][j] - VertStart;
		Mesh.CreateFaceVertexIndicesAttr().Set(VtTriVerts);
		pxr::VtArray<int> VtTriCounts(TriMeshCount);
		for (int32 i = 0; i < TriMeshCount; i++)
			VtTriCounts[i] = 3; // (This makes me die inside)
		Mesh.CreateFaceVertexCountsAttr().Set(VtTriCounts);

		// Subdivision scheme set to 'none' in UEUsdGeomTetMesh constructor, but we do it here anyway.
		Mesh.CreateSubdivisionSchemeAttr().Set(pxr::UsdGeomTokens->none);
	}
	else
	{
		UE_LOGF(LogUsd, Warning, "Failed to create UsdGeomMesh at path: '%ls' on USD stage: '%ls'.",
			*PrimPath, *Stage.GetRootLayer().GetRealPath());
	}

	// Promote to UEUsdGeomTetMesh.
	// This next line succeeds, but farts out a:
	//   TF_DIAGNOSTIC_CODING_ERROR_TYPE: Failed to find plugin for schema type 'UEUsdGeomTetMesh'
	// I think that means there's an issue with the plugInfo.json, and I think that means that if
	// USD was to happen upon a prim of this type, it might not know which plugin to load.  For 
	// our caching purposes, I think that's of no consequence; other than the fact that operator bool 
	// returns false for TetMesh, which I think may be related, but may not.  PointBased's operator 
	// bool does the same thing.  What's more is that we currently have no consumer for the tet 
	// topology, so this really doesn't matter.  At least not yet.
	pxr::UEUsdGeomTetMesh TetMesh = pxr::UEUsdGeomTetMesh::Define(Stage, Path);
	if (Tetrahedron->Num())
	{
		pxr::UsdPrim Prim = TetMesh.GetPrim();

		// Tetrahedra
		check(sizeof(pxr::GfVec4i) == sizeof(FIntVector4));
		const pxr::GfVec4i* GfTets = reinterpret_cast<const pxr::GfVec4i*>(&(*Tetrahedron)[TetMeshStart]);
		pxr::VtArray<pxr::GfVec4i> VtTets(TetMeshCount);
		for (int32 i = 0; i < TetMeshCount; i++) 
			VtTets[i] = GfTets[i] - pxr::GfVec4i(VertStart);
		TetMesh.CreateTetVertexIndicesAttr().Set(VtTets);
	}
	else
	{
		UE_LOGF(LogUsd, Warning, "Failed to create UEUsdGeomTetMesh at path: '%ls' on USD stage: '%ls'.",
			*PrimPath, *Stage.GetRootLayer().GetRealPath());
	}

	return true;
}

bool
UE::ChaosCachingUSD::WritePoints(
	UE::FUsdStage& Stage,
	const FString& PrimPath,
	const double Time,
	pxr::VtArray<pxr::GfVec3f>& VtPoints,
	pxr::VtArray<pxr::GfVec3f>& VtVels)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	pxr::GfMatrix4d GfMat(1.0);

	UE::FSdfPath Path(*PrimPath);
	pxr::UsdPrim Prim = Stage.GetPrimAtPath(Path);
	//UE::FUsdPrim Prim = PrimPath.IsEmpty() ? Stage.GetDefaultPrim() : Stage.GetPrimAtPath(UE::FSdfPath(*PrimPath));
	if (!Prim)
	{
		// We're authoring a prim that doesn't exist.  Use an override prim.
		Prim = Stage.OverridePrim(Path);
		//pxr::UsdGeomPointBased PointBased(Prim);
		//PointBased.CreatePointsAttr();
		//PointBased.CreateExtentAttr();

		UE::ChaosCachingUSDUtil::Private::DefineAncestorTransforms(Stage, Path);

		return UE::ChaosCachingUSDUtil::Private::SetPointsExtentAndPrimXform(
			Prim, GfMat, VtPoints, VtVels,
			Time == -TNumericLimits<double>::Max() ? pxr::UsdTimeCode::Default() : pxr::UsdTimeCode(Time));
	}
	else
	{
		// We're probably authoring an override prim, UsdGeomPointBased, or a UsdGeomPointInstancer.
		return UE::ChaosCachingUSDUtil::Private::SetPointsExtentAndPrimXform(
			Prim, GfMat, VtPoints, VtVels,
			Time == -TNumericLimits<double>::Max() ? pxr::UsdTimeCode::Default() : pxr::UsdTimeCode(Time));
	}
}

bool 
UE::ChaosCachingUSD::WritePoints(
	UE::FUsdStage& Stage, 
	const FString& PrimPath, 
	const double Time, 
	const FManagedArrayCollection& Collection, 
	const int32 StructureIndex)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	const TManagedArray<int32>* VertexStart =
		Collection.FindAttribute<int32>(
			"VertexStart", /*FGeometryCollection::GeometryGroup*/ FName("Geometry"));
	const TManagedArray<int32>* VertexCount =
		Collection.FindAttribute<int32>(
			"VertexCount", /*FGeometryCollection::GeometryGroup*/FName("Geometry"));

	const TManagedArray<FVector3f>* Vertex =
		Collection.FindAttribute<FVector3f>(
			"Vertex", "Vertices");

	int32 VertStart = 0;
	int32 VertCount = Vertex->Num();
	if (StructureIndex != INDEX_NONE)
	{
		if (!VertexStart->IsValidIndex(StructureIndex) ||
			!VertexCount->IsValidIndex(StructureIndex))
		{
			UE_LOGF(LogUsd, Error, "Invalid vertex index: %d, valid range [0, %d).", StructureIndex, VertexStart->Num());
			return false;
		}
		VertStart = (*VertexStart)[StructureIndex];
		VertCount = (*VertexCount)[StructureIndex];
	}

	check(sizeof(pxr::GfVec3f) == sizeof(FVector3f)); // reinterpret_cast works?
	pxr::VtArray<pxr::GfVec3f> VtPoints(VertCount); // VtArray is shared mem with copy-on-write.
	for (int32 i = 0; i < VertCount; i++)
		VtPoints[i] = *reinterpret_cast<const pxr::GfVec3f*>(&(*Vertex)[VertStart + i]);
	pxr::GfMatrix4d GfMat(1.0);

	UE::FSdfPath Path(*PrimPath);
	pxr::UsdPrim Prim = Stage.GetPrimAtPath(Path);
	if(!Prim)
	{
		// We're authoring a prim that doesn't exist.  Use an override prim.
		Prim = Stage.OverridePrim(Path);
		//pxr::UsdGeomPointBased PointBased(Prim);
		//PointBased.CreatePointsAttr();
		//PointBased.CreateExtentAttr();

		UE::ChaosCachingUSDUtil::Private::DefineAncestorTransforms(Stage, Path);

		return UE::ChaosCachingUSDUtil::Private::SetPointsExtentAndPrimXform(
			Prim, GfMat, VtPoints,
			Time == -TNumericLimits<double>::Max() ? pxr::UsdTimeCode::Default() : pxr::UsdTimeCode(Time));
	}
	else 
	{
		// We're probably authoring an override prim, UsdGeomPointBased, or a UsdGeomPointInstancer.
		return UE::ChaosCachingUSDUtil::Private::SetPointsExtentAndPrimXform(
			Prim, GfMat, VtPoints,
			Time == -TNumericLimits<double>::Max() ? pxr::UsdTimeCode::Default() : pxr::UsdTimeCode(Time));
	}
}

bool 
UE::ChaosCachingUSD::WritePoints(
	UE::FUsdStage& Stage, 
	const FString& PrimPath, 
	const double Time, 
	const TArray<Chaos::TVector<float, 3>>& Points, 
	const TArray<Chaos::TVector<float, 3>>& Vels)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	pxr::VtArray<pxr::GfVec3f> VtPoints(static_cast<size_t>(Points.Num()));
	for (int32 i = 0; i < Points.Num(); i++) VtPoints[i].Set(Points[i][0], Points[i][1], Points[i][2]);
	pxr::VtArray<pxr::GfVec3f> VtVels(static_cast<size_t>(Vels.Num()));
	for (int32 i = 0; i < Vels.Num(); i++) VtVels[i].Set(Vels[i][0], Vels[i][1], Vels[i][2]);

	return WritePoints(Stage, PrimPath, Time, VtPoints, VtVels);
}

bool UE::ChaosCachingUSD::WritePoints(
	UE::FUsdStage& Stage, 
	const FString& PrimPath, 
	const double Time, 
	const TArray<Chaos::TVector<float, 3>>& Points, 
	const TArray<Chaos::TVector<float, 3>>& Vels,
	const Chaos::FRange& PointsRange)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	const size_t NumPoints = PointsRange.Count;

	pxr::VtArray<pxr::GfVec3f> VtPoints(NumPoints);
	pxr::VtArray<pxr::GfVec3f> VtVels(NumPoints);

	int32 GlobalIndex = PointsRange.Start;
	for (int32 LocalIndex = 0; LocalIndex < NumPoints; ++LocalIndex, ++GlobalIndex)
	{
		VtPoints[LocalIndex].Set(Points[GlobalIndex][0], Points[GlobalIndex][1], Points[GlobalIndex][2]);
		VtVels[LocalIndex].Set(Vels[GlobalIndex][0], Vels[GlobalIndex][1], Vels[GlobalIndex][2]);
	}

	return WritePoints(Stage, PrimPath, Time, VtPoints, VtVels);
}

bool UE::ChaosCachingUSD::WriteActivations(
	UE::FUsdStage& Stage,
	const FString& PrimPath,
	const double Time,
	pxr::VtArray<float>& VtActivations)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	pxr::GfMatrix4d GfMat(1.0);

	UE::FSdfPath Path(*PrimPath);
	pxr::UsdPrim Prim = Stage.GetPrimAtPath(Path);
	if (!Prim)
	{
		// We're authoring a prim that doesn't exist.  Use an override prim.
		Prim = Stage.OverridePrim(Path);
	}
	return UE::ChaosCachingUSDUtil::Private::SetActivations(
			Prim, VtActivations,
			Time == -TNumericLimits<double>::Max() ? pxr::UsdTimeCode::Default() : pxr::UsdTimeCode(Time));
}

bool UE::ChaosCachingUSD::WriteActivations(
	UE::FUsdStage& Stage,
	const FString& PrimPath,
	const double Time,
	const TArray<float>& Activations,
	const Chaos::FRange& PointsRange)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	const size_t NumPoints = PointsRange.Count;

	pxr::VtArray<float> VtActivations(NumPoints);

	int32 GlobalIndex = PointsRange.Start;
	for (int32 LocalIndex = 0; LocalIndex < NumPoints; ++LocalIndex, ++GlobalIndex)
	{
		VtActivations[LocalIndex] = Activations[GlobalIndex];
	}

	return WriteActivations(Stage, PrimPath, Time, VtActivations);
}

bool
UE::ChaosCachingUSD::ReadTimeSamples(
	const UE::FUsdStage& Stage, 
	const FString& PrimPath, 
	const FString& AttrName, 
	TArray<double>& TimeSamples)
{
	UE::FSdfPath Path(*PrimPath);
	UE::FUsdPrim Prim = Stage.GetPrimAtPath(Path);
	if (!Prim)
	{
		UE_LOGF(LogUsd, Error, "No prim found at path '%ls'.", *PrimPath);
		return false;
	}
	UE::FUsdAttribute Attr = Prim.GetAttribute(*AttrName);
	if (!Attr)
	{
		UE_LOGF(LogUsd, Error, "No attribute '%ls' found on prim '%ls'.", *AttrName, *PrimPath);
		return false;
	}
	Attr.GetTimeSamples(TimeSamples);
	return true;
}

uint64
UE::ChaosCachingUSD::GetNumTimeSamples(
	const UE::FUsdStage& Stage,
	const FString& PrimPath,
	const FString& AttrName)
{
	UE::FSdfPath Path(*PrimPath);
	UE::FUsdPrim Prim = Stage.GetPrimAtPath(Path);
	if (!Prim)
	{
		UE_LOGF(LogUsd, Error, "No prim found at path '%ls'.", *PrimPath);
		return 0;
	}
	UE::FUsdAttribute Attr = Prim.GetAttribute(*AttrName);
	if (!Attr)
	{
		UE_LOGF(LogUsd, Error, "No attribute '%ls' found on prim '%ls'.", *AttrName, *PrimPath);
		return 0;
	}
	return Attr.GetNumTimeSamples();
}

bool 
UE::ChaosCachingUSD::ReadTimeSamples(
	const UE::FUsdStage& Stage, 
	const FString& PrimPath, 
	TArray<double>& TimeSamples)
{
	return ChaosCachingUSD::ReadTimeSamples(Stage, PrimPath, FString(pxr::UsdGeomTokens->points.GetText()), TimeSamples);
}

FString
UE::ChaosCachingUSD::GetPointsAttrName()
{
	FScopedUnrealAllocs UEAllocs; // Use UE memory allocator
	return FString(pxr::UsdGeomTokens->points.GetText());
}

FString
UE::ChaosCachingUSD::GetVelocityAttrName()
{
	FScopedUnrealAllocs UEAllocs; // Use UE memory allocator
	return FString(pxr::UsdGeomTokens->velocities.GetText());
}

bool 
UE::ChaosCachingUSD::GetBracketingTimeSamples(
	const UE::FUsdStage& Stage, 
	const FString& PrimPath, 
	const FString& AttrName, 
	const double TargetTime, 
	double* Lower, 
	double* Upper)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	UE::FSdfPath Path(*PrimPath);
	UE::FUsdPrim Prim = Stage.GetPrimAtPath(Path);
	if (!Prim)
	{
		UE_LOGF(LogUsd, Error, "No prim found at path '%ls'.", *PrimPath);
		return false;
	}
	UE::FUsdAttribute Attr = Prim.GetAttribute(*AttrName);
	if (!Attr)
	{
		UE_LOGF(LogUsd, Error, "No attribute '%ls' found on prim '%ls'.", *AttrName, *PrimPath);
		return false;
	}

	bool bHasTimeSamples = false;
	return static_cast<pxr::UsdAttribute&>(Attr).GetBracketingTimeSamples(
		TargetTime, Lower, Upper, &bHasTimeSamples) && bHasTimeSamples;
}

bool
UE::ChaosCachingUSD::ReadPoints(
	const UE::FUsdStage& Stage,
	const FString& PrimPath,
	const FString& AttrName,
	const double Time,
	pxr::VtArray<pxr::GfVec3f>& VtPoints)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator
	
	UE::FSdfPath Path(*PrimPath);
	UE::FUsdPrim Prim = Stage.GetPrimAtPath(Path);
	if (!Prim)
	{
		UE_LOGF(LogUsd, Error, "No prim found at path '%ls'.", *PrimPath);
		return false;
	}
	UE::FUsdAttribute Attr = Prim.GetAttribute(*AttrName);
	if (!Attr)
	{
		UE_LOGF(LogUsd, Error, "No attribute '%ls' found on prim '%ls'.", *AttrName, *PrimPath);
		return false;
	}

	// VtArray is copy-on-write. Be careful to use const API.
	if (Attr)
	{
		static_cast<pxr::UsdAttribute&>(Attr).Get(
			&VtPoints,
			Time == -TNumericLimits<double>::Max() ? pxr::UsdTimeCode::Default() : pxr::UsdTimeCode(Time));
		return true;
	}
	else
	{
		return false;
	}
}

bool 
UE::ChaosCachingUSD::ReadPoints(
	const UE::FUsdStage& Stage,
	const FString& PrimPath,
	const FString& AttrPath,
	const double Time,
	TArray<Chaos::TVector<Chaos::FRealSingle, 3>>& Points)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator
	pxr::VtArray<pxr::GfVec3f> VtPoints;
	if (ReadPoints(Stage, PrimPath, AttrPath, Time, VtPoints))
	{
		{
			FScopedUnrealAllocs UEAllocs; // Use UE memory allocator
			Points.SetNum(VtPoints.size());
		}
		int32 i = 0;
		for (pxr::VtArray<pxr::GfVec3f>::const_iterator it = VtPoints.cbegin(), itEnd = VtPoints.cend(); it != itEnd; ++it)
		{
			const pxr::GfVec3f& Pt = *it;
			Points[i++].Set(Pt[0], Pt[1], Pt[2]);
		}
		return true;
	}
	return false;
}

bool
UE::ChaosCachingUSD::ReadPoints(
	const UE::FUsdStage& Stage,
	const FString& PrimPath,
	const double Time,
	pxr::VtArray<pxr::GfVec3f>& VtPoints,
	pxr::VtArray<pxr::GfVec3f>& VtVels)
{
	//Velocity is optional, e.g. for quasistatics
	ChaosCachingUSD::ReadPoints(Stage, PrimPath, FString(pxr::UsdGeomTokens->velocities.GetText()), Time, VtVels);
	return ChaosCachingUSD::ReadPoints(Stage, PrimPath, FString(pxr::UsdGeomTokens->points.GetText()), Time, VtPoints);
}

bool
UE::ChaosCachingUSD::ReadMuscleActivation(
	const UE::FUsdStage& Stage,
	const FString& PrimPath,
	const FString& AttrName,
	const double Time,
	pxr::VtArray<float>& VtActivations)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	UE::FSdfPath Path(*PrimPath);
	UE::FUsdPrim Prim = Stage.GetPrimAtPath(Path);
	if (!Prim)
	{
		UE_LOGF(LogUsd, Error, "No prim found at path '%ls'.", *PrimPath);
		return false;
	}
	UE::FUsdAttribute Attr = Prim.GetAttribute(*AttrName);
	if (!Attr)
	{
		return false;
	}

	// VtArray is copy-on-write. Be careful to use const API.
	static_cast<pxr::UsdAttribute&>(Attr).Get(
		&VtActivations,
		Time == -TNumericLimits<double>::Max() ? pxr::UsdTimeCode::Default() : pxr::UsdTimeCode(Time));

	return true;
}

bool
UE::ChaosCachingUSD::ReadMuscleActivation(
	const UE::FUsdStage& Stage,
	const FString& PrimPath,
	const double Time,
	pxr::VtArray<float>& VtActivations)
{
	return ChaosCachingUSD::ReadMuscleActivation(Stage, PrimPath, FString("activation"), Time, VtActivations);
}

bool
UE::ChaosCachingUSD::DeletePointsInTimeRange(UE::FUsdStage& Stage, const FString& PrimPath, const double StartTime, const double EndTime)
{
	FScopedUsdAllocs UsdAllocs; // Use USD memory allocator

	UE::FSdfPath Path(*PrimPath);
	pxr::UsdPrim Prim = Stage.GetPrimAtPath(Path);
	if (!Prim)
	{
		// We're authoring a prim that doesn't exist. Use an override prim.
		Prim = Stage.OverridePrim(Path);
		UE::ChaosCachingUSDUtil::Private::DefineAncestorTransforms(Stage, Path);
	}
	return UE::ChaosCachingUSDUtil::Private::DeletePointsInTimeRange(
			Prim, StartTime, EndTime);
}
#endif // USE_USD_SDK
