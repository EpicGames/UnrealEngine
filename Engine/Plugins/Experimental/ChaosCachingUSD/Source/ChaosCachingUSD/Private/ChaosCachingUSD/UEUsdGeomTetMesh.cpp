// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCachingUSD/UEUsdGeomTetMesh.h"
#include "ChaosCachingUSD/UEUsdGeomTokens.h"

#if USE_USD_SDK

#include "USDMemory.h"
#include "USDIncludesStart.h"
	#include "pxr/usd/sdf/assetPath.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE

UEUsdGeomTetMesh::UEUsdGeomTetMesh(const UsdPrim& Prim)
	: UsdGeomMesh(Prim)
{
	if(pxr::UsdAttribute SubdivisionAttr = UsdGeomMesh::CreateSubdivisionSchemeAttr())
	{
		SubdivisionAttr.Set(pxr::UsdGeomTokens->none);
	}
}

UEUsdGeomTetMesh::UEUsdGeomTetMesh(const UsdSchemaBase& SchemaObj)
	: UsdGeomMesh(SchemaObj)
{
	if(pxr::UsdAttribute SubdivisionAttr = UsdGeomMesh::CreateSubdivisionSchemeAttr())
	{
		SubdivisionAttr.Set(pxr::UsdGeomTokens->none);
	}
}

UEUsdGeomTetMesh::~UEUsdGeomTetMesh()
{}

UEUsdGeomTetMesh
UEUsdGeomTetMesh::Get(const UsdStagePtr &Stage, const SdfPath &Path)
{
	if (!Stage)
	{
		TF_CODING_ERROR("Invalid stage");
		return UEUsdGeomTetMesh();
	}
	return UEUsdGeomTetMesh(Stage->GetPrimAtPath(Path));
}

UEUsdGeomTetMesh
UEUsdGeomTetMesh::Define(const UsdStagePtr &Stage, const SdfPath &Path)
{
	static TfToken UsdPrimTypeName("UETetMesh");
	if(!Stage)
	{
		TF_CODING_ERROR("Invalid stage");
		return UEUsdGeomTetMesh();
	}
	return UEUsdGeomTetMesh(Stage->DefinePrim(Path, UsdPrimTypeName));
}

UsdSchemaKind
UEUsdGeomTetMesh::_GetSchemaKind() const
{
	return UEUsdGeomTetMesh::schemaKind;
}

const TfType &
UEUsdGeomTetMesh::_GetStaticTfType()
{
	static TfType Type = TfType::Find<UEUsdGeomTetMesh>();
	return Type;
}

bool
UEUsdGeomTetMesh::_IsTypedSchema()
{
	static bool bIsTyped = _GetStaticTfType().IsA<UsdTyped>();
	return bIsTyped;
}

const TfType &
UEUsdGeomTetMesh::_GetTfType() const
{
	return _GetStaticTfType();
}

UsdAttribute
UEUsdGeomTetMesh::GetTetVertexIndicesAttr() const
{
	return GetPrim().GetAttribute(UEUsdGeomTokens->tetVertexIndices);
}

UsdAttribute
UEUsdGeomTetMesh::CreateTetVertexIndicesAttr(VtValue const &DefaultValue, bool bWriteSparsely) const
{
	return UsdSchemaBase::_CreateAttr(
		UEUsdGeomTokens->tetVertexIndices,
		SdfValueTypeNames->Int4Array,
		false,    // custom
		SdfVariabilityVarying,
		DefaultValue,
		bWriteSparsely);
}

UsdAttribute 
UEUsdGeomTetMesh::GetTetOrientationAttr() const
{
	return GetPrim().GetAttribute(UEUsdGeomTokens->tetOrientation);
}

UsdAttribute 
UEUsdGeomTetMesh::CreateTetOrientationAttr(VtValue const& DefaultValue, bool bWriteSparsely) const
{
	return UsdSchemaBase::_CreateAttr(
		UEUsdGeomTokens->tetOrientation,
		SdfValueTypeNames->Token,
		false,	// custom
		SdfVariabilityUniform,
		DefaultValue,
		bWriteSparsely);
}

namespace {
	static inline TfTokenVector
	_ConcatenateAttributeNames(const TfTokenVector& Left, const TfTokenVector& Right)
	{
		TfTokenVector Result;
		Result.reserve(Left.size() + Right.size());
		Result.insert(Result.end(), Left.begin(), Left.end());
		Result.insert(Result.end(), Right.begin(), Right.end());
		return Result;
	}
} // namespace

const TfTokenVector&
UEUsdGeomTetMesh::GetSchemaAttributeNames(bool bIncludeInerited)
{
	static TfTokenVector LocalNames = {
		UEUsdGeomTokens->tetOrientation,
		UEUsdGeomTokens->tetVertexIndices,
	};
	static TfTokenVector AllNames =
		_ConcatenateAttributeNames(
		  UsdGeomPointBased::GetSchemaAttributeNames(true),
		  LocalNames);
	return bIncludeInerited ? AllNames : LocalNames;
}

// ===================================================================== //
// Feel free to add custom code below this line. It will be preserved by
// the code generator.
// ===================================================================== //
// --(BEGIN CUSTOM CODE)--

PXR_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
