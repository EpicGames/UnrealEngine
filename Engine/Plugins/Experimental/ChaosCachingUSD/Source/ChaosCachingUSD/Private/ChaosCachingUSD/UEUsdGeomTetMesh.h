// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#if USE_USD_SDK

#include "USDIncludesStart.h"
	#include "pxr/base/gf/vec4i.h"
	#include "pxr/base/tf/type.h"
	#include "pxr/base/vt/value.h"

	#include "pxr/usd/usdGeom/mesh.h"
	#include "pxr/usd/usd/prim.h"
	#include "pxr/usd/usd/stage.h"
	#include "pxr/usd/usdGeom/tokens.h"
	#include "pxr/usd/usd/timeCode.h"
#include "USDIncludesEnd.h"

PXR_NAMESPACE_OPEN_SCOPE

class SdfAssetPath;

/**
 * \class UEUsdGeomTetMesh
 *
 * A Tetrahedral mesh prim, derived from \c UsdGeomMesh, which is used for 
 * drawing the exterior hull of the tetrahedron mesh.  This class adds a
 * topology array for tetrahedra consisting of \c GfVec4i, and an attribute
 * that specifies the tetrahedron orientation/winding order.
 * 
 * The constructor defines and sets the subdivision scheme attribute to `none`
 * so that by default the geometry rendered by the \c UsdGeomMesh base class
 * is polygonal.
 */
class UEUsdGeomTetMesh : public UsdGeomMesh
{
public:
    static const UsdSchemaKind schemaKind = UsdSchemaKind::ConcreteTyped;                                                                          

    /** Construct a UEUsdGeomTetMesh from a UsdPrim \p prim. */
    explicit UEUsdGeomTetMesh(const UsdPrim& Prim=UsdPrim());
    explicit UEUsdGeomTetMesh(const UsdSchemaBase& SchemaObj);

    virtual ~UEUsdGeomTetMesh();

    static const TfTokenVector &
    GetSchemaAttributeNames(bool bIncludeInherited=true);

    static UEUsdGeomTetMesh
    Get(const UsdStagePtr &Stage, const SdfPath &Path);

    static UEUsdGeomTetMesh
    Define(const UsdStagePtr &Stage, const SdfPath &Path);
   
protected:
    UsdSchemaKind _GetSchemaKind() const override;

private:
    // Needs to invoke _GetStaticTfType.
    friend class UsdSchemaRegistry;
    static const TfType &_GetStaticTfType();

    static bool _IsTypedSchema();

    // Override SchemaBase virtuals.
    virtual const TfType &_GetTfType() const;

public:
    /**
	 * Tetrahedron indices. Each element of this array contains the
     * indices of the vertices of a tetrahedron. The points are authored
     * into the "points" attribute, just as they are for other PointBased    
     * prims.
     *
     *   C++ Type: VtArray<GfVec4i>
     *   Usd Type: SdfValueTypeNames->Int4Array
     *   Variability: SdfVariabilityVarying
     *   Fallback Value: No Fallback
	 */
    UsdAttribute GetTetVertexIndicesAttr() const;
    
    /**
	 * See GetTetVertexIndicesAttr(), and also
     * \ref Usd_Create_Or_Get_Property for when to use Get vs Create.
     * If specified, author \p DefaultValue as the attribute's default,
     * sparsely (when it makes sense to do so) if \p WriteSparsely is \c true -
     * the default for \p writeSparsely is \c false.
	 */
    UsdAttribute CreateTetVertexIndicesAttr(VtValue const &DefaultValue = VtValue(), bool WriteSparsely=false) const;
    
public:
	/**
	 * Tetrahedron orientation. Given [A,B,C,D] if your right hand 
	 * is at A, if your fingers wrap to A-B, then A-C, and your thumb 
	 * points at D.
	 */
	UsdAttribute GetTetOrientationAttr() const;

	/**
	* See GetTetOrientationAttr().
	*/
	UsdAttribute CreateTetOrientationAttr(VtValue const& DefaultValue = VtValue(), bool WriteSparsely = false) const;
};

PXR_NAMESPACE_CLOSE_SCOPE
#endif // USE_USD_SDK