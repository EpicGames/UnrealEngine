// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCachingUSD/UEUsdGeomTokens.h"

#if USE_USD_SDK

PXR_NAMESPACE_OPEN_SCOPE

UEUsdGeomTokensType::UEUsdGeomTokensType()
    : tetOrientation("tetOrientation", TfToken::Immortal)
    , tetVertexIndices("tetVertexIndices", TfToken::Immortal)
    , allTokens({
            tetOrientation,
            tetVertexIndices
        })
{}

TfStaticData<UEUsdGeomTokensType> UEUsdGeomTokens;

PXR_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
