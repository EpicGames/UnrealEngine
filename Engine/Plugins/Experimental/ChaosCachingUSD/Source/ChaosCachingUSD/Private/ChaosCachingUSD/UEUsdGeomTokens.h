// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if USE_USD_SDK

#include "USDIncludesStart.h"
	#include "pxr/base/tf/staticData.h"
	#include "pxr/base/tf/token.h"
#include "USDIncludesEnd.h"

#include <vector>

PXR_NAMESPACE_OPEN_SCOPE

struct UEUsdGeomTokensType 
{
	CHAOSCACHINGUSD_API UEUsdGeomTokensType();

    /// \brief "tetOrientation"
    ///
    /// UEUsdGeomTetMesh
    const TfToken tetOrientation;
    /// \brief "tetVetexIndices"
    ///
    /// UEUsdGeomTetMesh
    const TfToken tetVertexIndices;
    /// A vector of all of the tokens listed above.
    const std::vector<TfToken> allTokens;
};

/// \var UEUsdGeomTokens
///
/// A global variable for access to static tokens.
extern CHAOSCACHINGUSD_API TfStaticData<UEUsdGeomTokensType> UEUsdGeomTokens;

PXR_NAMESPACE_CLOSE_SCOPE

#endif // USE_USD_SDK
