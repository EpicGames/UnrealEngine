// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCachingUSD/Util.h"

#if USE_USD_SDK

#include "USDIncludesStart.h"
	#include "pxr/usd/usdGeom/pointBased.h"
	#include "pxr/usd/usdGeom/xform.h"
#include "USDIncludesEnd.h"

void 
UE::ChaosCachingUSDUtil::Private::SaveStage(
	const pxr::UsdStageRefPtr& Stage,
	const pxr::UsdTimeCode& FirstFrame,
	const pxr::UsdTimeCode& LastFrame)
{
	FScopedUsdAllocs UEAllocs; // Use USD memory allocator
	if (FirstFrame.IsNumeric() && LastFrame.IsNumeric())
	{
		// Only expand the existing frame range.
		const double StageStart = Stage->GetStartTimeCode();
		const double StageEnd = Stage->GetEndTimeCode();
		const bool HasRange = Stage->HasAuthoredTimeCodeRange();
		if (!HasRange || FirstFrame.GetValue() < StageStart)
		{
			Stage->SetStartTimeCode(FirstFrame.GetValue());
		}
		if (!HasRange || LastFrame.GetValue() > StageEnd)
		{
			Stage->SetEndTimeCode(LastFrame.GetValue());
		}
	}
	Stage->GetRootLayer()->Save();
}

void 
UE::ChaosCachingUSDUtil::Private::DefineAncestorTransforms(
	pxr::UsdStageRefPtr& Stage,
	const pxr::SdfPath& PrimPath)
{
	FScopedUsdAllocs UEAllocs; // Use USD memory allocator
	pxr::SdfPath Path(PrimPath.GetParentPath());
	//while (Path != pxr::SdfPath::EmptyPath())
	while (!Path.IsAbsoluteRootPath())
	{
		pxr::UsdGeomXform::Define(Stage, Path);
		Path = Path.GetParentPath();
	}
}

void
UE::ChaosCachingUSDUtil::Private::ComputeExtent(
	const pxr::VtArray<pxr::GfVec3f>& Points,
	pxr::VtVec3fArray& Extent)
{
	FScopedUsdAllocs UEAllocs; // Use USD memory allocator
	pxr::GfRange3f Bounds;
	for (size_t i = 0; i < Points.size(); i++)
	{
		Bounds = Bounds.UnionWith(Points[i]);
	}
	Extent[0] = Bounds.GetMin();
	Extent[1] = Bounds.GetMax();
}

bool 
UE::ChaosCachingUSDUtil::Private::SetPointsExtentAndPrimXform(
	pxr::UsdPrim& Prim, 
	const pxr::GfMatrix4d& PrimXf, 
	const pxr::VtArray<pxr::GfVec3f>& Points, 
	const pxr::VtArray<pxr::GfVec3f>& Vels, 
	const pxr::UsdTimeCode& Time)
{
	FScopedUsdAllocs UEAllocs; // Use USD memory allocator

	pxr::VtVec3fArray Extent(2);
	UE::ChaosCachingUSDUtil::Private::ComputeExtent(Points, Extent);

	pxr::UsdGeomPointBased PointBased(Prim);
	PointBased.MakeMatrixXform().Set(PrimXf, Time);
	PointBased.CreateExtentAttr().Set(Extent, Time);
	if (!PointBased.CreatePointsAttr().Set(Points, Time)) return false;
	if (!PointBased.CreateVelocitiesAttr().Set(Vels, Time)) return false;
	return true;
}

bool 
UE::ChaosCachingUSDUtil::Private::SetPointsExtentAndPrimXform(
	pxr::UsdPrim& Prim,
	const pxr::GfMatrix4d& PrimXf,
	const pxr::VtArray<pxr::GfVec3f>& Points,
	const pxr::UsdTimeCode& Time)
{
	FScopedUsdAllocs UEAllocs; // Use USD memory allocator
	pxr::VtArray<pxr::GfVec3f> Vels;
	return UE::ChaosCachingUSDUtil::Private::SetPointsExtentAndPrimXform(Prim, PrimXf, Points, Vels, Time);
}

bool
UE::ChaosCachingUSDUtil::Private::SetActivations(
	pxr::UsdPrim& Prim,
	const pxr::VtArray<float>& Activations,
	const pxr::UsdTimeCode& Time)
{
	FScopedUsdAllocs UEAllocs; // Use USD memory allocator

	if (auto ActivationAttr = Prim.CreateAttribute(
		pxr::TfToken("activation"), pxr::SdfValueTypeNames->FloatArray))
	{
		ActivationAttr.Set(Activations, Time);
		return true;
	}
	return false;
}

bool
UE::ChaosCachingUSDUtil::Private::DeletePointsInTimeRange(pxr::UsdPrim& Prim, float StartTime, float EndTime)
{
	FScopedUsdAllocs UEAllocs; // Use USD memory allocator
	
	if (!Prim)
	{
		return false;
	}

	pxr::UsdGeomPointBased PointBased(Prim);

	DeleteUsdAttributeInTimeRange(PointBased.MakeMatrixXform(), StartTime, EndTime);
	DeleteUsdAttributeInTimeRange(PointBased.GetExtentAttr(), StartTime, EndTime);
	DeleteUsdAttributeInTimeRange(PointBased.GetVelocitiesAttr(), StartTime, EndTime);
	DeleteUsdAttributeInTimeRange(Prim.GetAttribute(pxr::TfToken("activation")), StartTime, EndTime);
	return DeleteUsdAttributeInTimeRange(PointBased.GetPointsAttr(), StartTime, EndTime);
}

bool
UE::ChaosCachingUSDUtil::Private::DeleteUsdAttributeInTimeRange(pxr::UsdAttribute Attribute, float StartTime, float EndTime)
{
	if (!Attribute)
	{
		return false;
	}
	// Get all time samples for the attribute
	std::vector<double> TimeSamples;
	if (!Attribute.GetTimeSamples(&TimeSamples))
	{
		return false; // No time samples available
	}

	// Iterate over time samples and remove those within the specified range
	for (double TimeSample : TimeSamples)
	{
		if (TimeSample >= StartTime && TimeSample <= EndTime)
		{
			Attribute.ClearAtTime(TimeSample); // Remove the specific time sample
		}
	}
	return true;
}
#endif // USE_USD_SDK
