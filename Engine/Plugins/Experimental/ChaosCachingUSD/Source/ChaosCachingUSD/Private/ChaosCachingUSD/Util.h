// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#if USE_USD_SDK

#include "USDMemory.h"
#include "USDIncludesStart.h"
	#include "pxr/usd/sdf/usdFileFormat.h"
	#include "pxr/usd/sdf/usdaFileFormat.h"
	#include "pxr/usd/sdf/usdcFileFormat.h"

	#include "pxr/usd/usd/tokens.h"
	#include "pxr/usd/usdUtils/stitchClips.h"

	#include "pxr/base/tf/fileUtils.h"
	#include "pxr/base/tf/pathUtils.h"

	#include "pxr/base/vt/array.h"
	#include "pxr/base/vt/dictionary.h"
	#include "pxr/base/vt/value.h"
#include "USDIncludesEnd.h"

namespace UE::ChaosCachingUSDUtil::Private {

	/**
	 * Convert from \c pxr::VtArray<T1> to \c TArray<T2>, where \p T1 and \p T2 are binary compatible.
	 *
	 * Note that \c VtArray is shared copy-on-write, which makes them trivially copied. This function
	 * copies memory, which should be avoided.
	 */
	template <class T1, class T2>
	void CastConvert(const pxr::VtArray<T1>& VtArray, TArray<T2>& UEArray)
	{
		check(sizeof(T1) == sizeof(T2)); // reinterpret_cast works?
		check(VtArray.size() <= TNumericLimits<int32>::Max()); // int32 index works?
		FScopedUnrealAllocs UEAllocs; // Use UE memory allocator
		UEArray.SetNum(VtArray.size());
		int32 i = 0;
		for (typename pxr::VtArray<T1>::const_iterator it = VtArray.cbegin(), itEnd = VtArray.cend();
			it != itEnd; ++it)
		{
			UEArray[i++] = *reinterpret_cast<const T2*>(&*it);
		}
	}

	/**
	 * Linearly interpolate values in \p VtArray1 and \p VtArray2 by interpolant \p Alpha ([0,1]), 
	 * converting from \p T1 to \p T2, where \p T1 and \p T2 are binary compatible.
	 *
	 * Note that \c VtArray is shared copy-on-write, which makes them trivially copied. This function
	 * copies memory, which should be avoided.
	 */
	template <class T1, class T2>
	void CastLerp(const pxr::VtArray<T1>& VtArray1, const pxr::VtArray<T1>& VtArray2, TArray<T2>& UEArray, const float Alpha)
	{
		check(sizeof(T1) == sizeof(T2)); // reinterpret_cast works?
		check(VtArray1.size() <= TNumericLimits<int32>::Max()); // int32 index works?
		check(VtArray1.size() == VtArray2.size());
		FScopedUnrealAllocs UEAllocs; // Use UE memory allocator
		UEArray.SetNum(VtArray1.size());
		int32 i = 0;
		if (Alpha <= TNumericLimits<float>::Min())
		{
			for (typename pxr::VtArray<T1>::const_iterator
				it1 = VtArray1.cbegin(), itEnd1 = VtArray1.end();
				it1 != itEnd1; ++it1)
			{
				UEArray[i++] = *reinterpret_cast<const T2*>(&*it1);
			}
		}
		else if (Alpha >= 1.0 - TNumericLimits<float>::Min())
		{
			for (typename pxr::VtArray<T1>::const_iterator
				it2 = VtArray2.cbegin(), itEnd2 = VtArray2.end();
				it2 != itEnd2; ++it2)
			{
				UEArray[i++] = *reinterpret_cast<const T2*>(&*it2);
			}
		}
		else
		{
			for (typename pxr::VtArray<T1>::const_iterator
				it1 = VtArray1.cbegin(), itEnd1 = VtArray1.end(),
				it2 = VtArray2.cbegin(), itEnd2 = VtArray2.cend();
				it1 != itEnd1 && it2 != itEnd2; ++it1, ++it2)
			{
				UEArray[i++] = 
					(1.0 - Alpha) * *reinterpret_cast<const T2*>(&*it1) + 
					       Alpha  * *reinterpret_cast<const T2*>(&*it2);
			}
		}
	}

	void SaveStage(const pxr::UsdStageRefPtr& Stage, const pxr::UsdTimeCode& FirstFrame, const pxr::UsdTimeCode &LastFrame);
	void DefineAncestorTransforms(pxr::UsdStageRefPtr& Stage, const pxr::SdfPath& PrimPath);
	void ComputeExtent(const pxr::VtArray<pxr::GfVec3f>& Points, pxr::VtVec3fArray& Extent);
	bool SetPointsExtentAndPrimXform(pxr::UsdPrim& Prim, const pxr::GfMatrix4d& PrimXf, const pxr::VtArray<pxr::GfVec3f>& Points, const pxr::VtArray<pxr::GfVec3f>& Vels, const pxr::UsdTimeCode& Time = pxr::UsdTimeCode::Default());
	bool SetPointsExtentAndPrimXform(pxr::UsdPrim& Prim, const pxr::GfMatrix4d& PrimXf, const pxr::VtArray<pxr::GfVec3f>& Points, const pxr::UsdTimeCode& Time = pxr::UsdTimeCode::Default());
	bool SetActivations(pxr::UsdPrim& Prim, const pxr::VtArray<float>& Activations, const pxr::UsdTimeCode& Time);
	bool DeletePointsInTimeRange(pxr::UsdPrim& Prim, float StartTime, float EndTime);
	bool DeleteUsdAttributeInTimeRange(pxr::UsdAttribute Attribute, float StartTime, float EndTime);
} // namespace UE::ChaosCachingUSDUtil::Private

#endif // USE_USD_SDK
