// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"

UE_DEPRECATED(5.7, "Use LogUsd instead")
DECLARE_LOG_CATEGORY_EXTERN(LogChaosCacheUSD, Verbose, All);

class FChaosCachingUSDModule : public IModuleInterface
{
	//~ Begin IModuleInterface API
	virtual void StartupModule() override;
	//~ End IModuleInterface API

};
