// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if USE_USD_SDK

#include "Chaos/Range.h"
#include "Chaos/Vector.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArray.h"
#include "Math/UnrealMathUtility.h"
#include "UsdWrappers/UsdStage.h"

#include "USDIncludesStart.h"
	#include "pxr/base/gf/vec3f.h"
	#include "pxr/base/vt/array.h"
#include "USDIncludesEnd.h"

namespace UE::ChaosCachingUSD
{

	inline bool ValuesDiffer(const pxr::VtArray<pxr::GfVec3f>& A, const pxr::VtArray<pxr::GfVec3f>& B, const float Tolerance = 1.0e-8, const uint64 stride = 1)
	{
		if (A.size() != B.size()) return true;
		for (size_t i = 0; i < A.size(); i += stride)
		{
			for (size_t j = 0; j < 3; j++)
			{
				if (!FMath::IsNearlyEqual(A[i][j], B[i][j], Tolerance))
					return true;
			}
		}
		return false;
	}

	/** Create a new stage, resident in the USD stage cache. */
	CHAOSCACHINGUSD_API bool NewStage(const FString& StageName, UE::FUsdStage& Stage);
	/** Opens an existing stage, resident in the USD stage cache. */
	CHAOSCACHINGUSD_API bool OpenStage(const FString& StageName, UE::FUsdStage& Stage);

	/** Save the stage and update the frame range (if not -DBL_MAX). */
	CHAOSCACHINGUSD_API bool SaveStage(UE::FUsdStage& Stage, const double FirstFrame, const double LastFrame);

	/** Close and remove a stage from the USD stage cache. */
	CHAOSCACHINGUSD_API bool CloseStage(const UE::FUsdStage& Stage);
	CHAOSCACHINGUSD_API bool CloseStage(const FString& StageName);

	/**
	 * "Value Clips" is USD's concept of splitting data up into multiple 
	 * files, the it looks like this:
	 *
	 *		path/to/file.usd - top level root or "parent" file that contains references to other files
	 *		path/to/file.topology.usd - unvarying data goes here
	 *		path/to/file.manifest.usd - we don't use this file
	 *		path/to/file.#.usd - data for frame # goes here, where # is some number.
	 *
	 * Writing value clips scenes requires juggling multiple stages. Reading value clips scenes
	 * should be somewhat simpler as USD's composition should pick up file references.
	 */

	/** Given 'path/to/file.usd', yields \p TopologyName 'path/to/file.topology.usd' and \p TimeVaryingTemplate 'path/to/file.###.###.usd'. */
	CHAOSCACHINGUSD_API void
	GenerateValueClipStageNames(const FString& ParentName, FString& TopologyName, FString& TimeVaryingTemplate);
	/** Given 'path/to/file.###.###.usd' and 1.23, yields 'path/to/file.001.230.usd'. */
	CHAOSCACHINGUSD_API FString
	GenerateValueClipTimeVaryingStageName(const FString& TimeVaryingTemplate, const double Time);

	/** Create new parent and topology stages for value clips. */
	CHAOSCACHINGUSD_API bool
	NewValueClipsStages(
		const FString& ParentStageName,
		const FString& TopologyStageName,
		UE::FUsdStage& ParentStage,
		UE::FUsdStage& TopologyStage);
	/** Create new value clips frame stage. Be sure to close these stages. */
	CHAOSCACHINGUSD_API bool
	NewValueClipsFrameStage(
		const FString& TimeVaryingStageTemplate,
		const double Time,
		FString& FrameStageName,
		UE::FUsdStage& FrameStage);
	/** 
	 * Init layers and metadata for value clips. 
	 * 
	 * \p ParentStage - the stage that serves as the hub and references others.
	 * \p TopologyStage - stage that will hod unvarying scene data.
	 * \p ParentStageName
	 * \p TopologyStageName
	 * \p TimeVaryingStageTemplate - template pattern for per-frame USD data files.
	 * \p PrimPaths - the prim(s) to author value clips metadata on.  It will apply to all prims 
	 *    below this point in the hierarchy.  Authoring the root scope is not allowed by USD.
	 * \p StartTime - the first time sample that should be available at this point in time.
	 * \p EndTime - the last time sample that should be available at this point in time.
	 * \p Stride - the size of the steps taken between \p StartTime and \p EndTime.
	 */
	CHAOSCACHINGUSD_API bool
	InitValueClipsTemplate(
		UE::FUsdStage& ParentStage,
		UE::FUsdStage& TopologyStage,
		const FString& ParentStageName,
		const FString& TopologyStageName,
		const FString& TimeVaryingStageTemplate,
		const TArray<FString>& PrimPaths,
		const double StartTime,
		const double EndTime,
		const double Stride);
	/** Helper function that opens/retreives parent and topology stages from cache. */
	CHAOSCACHINGUSD_API bool
	InitValueClipsTemplate(
		const FString& ParentStageName,
		const FString& TopologyStageName,
		const FString& TimeVaryingStageTemplate,
		const TArray<FString>& PrimPaths,
		const double StartTime,
		const double EndTime,
		const double Stride);

	/**
	 * Define \c UEUsdGeomTetMesh at \c PrimPath with ancenstor transforms, and authors points at
	 * usd "default" time from tet mesh \p StructureIndex (or all if \c INDEX_NONE) from \c Collection.
	 */
	CHAOSCACHINGUSD_API bool WriteTetMesh(UE::FUsdStage& Stage, const FString& PrimPath, const FManagedArrayCollection& Collection, const int32 StructureIndex = INDEX_NONE);

	/**
	 * Define \c UsdGeomPointBased at \c PrimPath with ancestor transforms, and author points at 
	 * \p Time (or usd "default" time if \c -DBL_MAX) from \p Collection vertices from geometry 
	 * \p StructureIndex (or all if \c StructureIndex == \c INDEX_NONE).
	 */
	CHAOSCACHINGUSD_API bool WritePoints(UE::FUsdStage& Stage, const FString& PrimPath, const double Time, const FManagedArrayCollection& Collection, const int32 StructureIndex = INDEX_NONE);
	/** Write points and velocities to \c UsdGeomPointBased at \c PrimPath. */
	CHAOSCACHINGUSD_API bool WritePoints(UE::FUsdStage& Stage, const FString& PrimPath, const double Time, pxr::VtArray<pxr::GfVec3f>& VtPoints, pxr::VtArray<pxr::GfVec3f>& VtVels);
	/** Copy \p Points and \p Vels to \c VtArray (with USD memory allocator), then write to USD stage. */
	CHAOSCACHINGUSD_API bool WritePoints(UE::FUsdStage& Stage, const FString& PrimPath, const double Time, const TArray<Chaos::TVector<float, 3>>& Points, const TArray<Chaos::TVector<float, 3>>& Vels);

	/** Copy \p Points and \p Vels within the specified indices range to \c VtArray (with USD memory allocator), then write to USD stage. */
	CHAOSCACHINGUSD_API bool WritePoints(UE::FUsdStage& Stage, const FString& PrimPath, const double Time, const TArray<Chaos::TVector<float, 3>>& Points, const TArray<Chaos::TVector<float, 3>>& Vels, const Chaos::FRange& PointsRange);
	
	CHAOSCACHINGUSD_API bool WriteActivations(UE::FUsdStage& Stage, const FString& PrimPath, const double Time, pxr::VtArray<float>& VtActivations);
	CHAOSCACHINGUSD_API bool WriteActivations(UE::FUsdStage& Stage, const FString& PrimPath, const double Time, const TArray<float>& Activations, const Chaos::FRange& PointsRange);

	/** Get time samples for an attribute. */
	CHAOSCACHINGUSD_API bool ReadTimeSamples(const UE::FUsdStage& Stage, const FString& PrimPath, const FString& AttrName, TArray<double>& TimeSamples);
	/** Get time samples for the points attribute. */
	CHAOSCACHINGUSD_API bool ReadTimeSamples(const UE::FUsdStage& Stage, const FString& PrimPath, TArray<double>& TimeSamples);
	CHAOSCACHINGUSD_API uint64 GetNumTimeSamples(const UE::FUsdStage& Stage, const FString& PrimPath, const FString& AttrName);

	CHAOSCACHINGUSD_API FString GetPointsAttrName();
	CHAOSCACHINGUSD_API FString GetVelocityAttrName();
	CHAOSCACHINGUSD_API bool GetBracketingTimeSamples(const UE::FUsdStage& Stage, const FString& PrimPath, const FString& AttrName, const double TargetTime, double* Lower, double* Upper);

	/** Get points from an attribute. Default time is used if \p Time is \c -DBL_MAX. */
	CHAOSCACHINGUSD_API bool ReadPoints(const UE::FUsdStage& Stage, const FString& PrimPath, const FString& AttrPath, const double Time, pxr::VtArray<pxr::GfVec3f>& Points);
	CHAOSCACHINGUSD_API bool ReadPoints(const UE::FUsdStage& Stage, const FString& PrimPath, const FString& AttrPath, const double Time, TArray<Chaos::TVector<Chaos::FRealSingle,3>>& Points);

	/** Get points from the points attribute. Default time is used if \p Time is \c -DBL_MAX. */
	CHAOSCACHINGUSD_API bool ReadPoints(const UE::FUsdStage& Stage, const FString& PrimPath, const double Time, pxr::VtArray<pxr::GfVec3f>& Points, pxr::VtArray<pxr::GfVec3f>& VtVels);

	CHAOSCACHINGUSD_API bool ReadMuscleActivation(const UE::FUsdStage& Stage, const FString& PrimPath, const FString& AttrName, const double Time, pxr::VtArray<float>& VtActivations);
	CHAOSCACHINGUSD_API bool ReadMuscleActivation(const UE::FUsdStage& Stage, const FString& PrimPath, const double Time, pxr::VtArray<float>& VtActivations);
	
	CHAOSCACHINGUSD_API bool DeletePointsInTimeRange(UE::FUsdStage& Stage, const FString& PrimPath, const double StartTime, const double EndTime);
	
} // UE::ChaosCachingUSD
#endif // USE_USD_SDK
